// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditMode.h"

#include "ContextObjectStore.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EditorModeManager.h"
#include "Misc/AssertionMacros.h"
#include "Tools/Modes.h"
#include "Tools/UEdMode.h"
#include "UObject/ObjectHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationEditMode)

class FText;
class IPersonaPreviewScene;

bool UAnimationEditModeContext::GetCameraTarget(FSphere& OutTarget) const
{
	check(EditMode);
	return EditMode->GetCameraTarget(OutTarget);
}

IPersonaPreviewScene& UAnimationEditModeContext::GetAnimPreviewScene() const
{
	check(EditMode);
	return EditMode->GetAnimPreviewScene();
}

void UAnimationEditModeContext::GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const
{
	check(EditMode);
	return EditMode->GetOnScreenDebugInfo(OutDebugInfo);
};

FAnimationEditMode::FAnimationEditMode()
	: AnimationEditModeContext(UAnimationEditModeContext::CreateFor(this))
{
}

void FAnimationEditMode::Enter()
{
	FEdMode::Enter();

	if (const UEdMode* EdMode = GetModeManager()->GetActiveScriptableMode(Info.ID))
	{
		UContextObjectStore* ContextObjectStore = EdMode->GetInteractiveToolsContext()->ContextObjectStore;
		ContextObjectStore->AddContextObject(AnimationEditModeContext.Get());
	}
}

void FAnimationEditMode::Exit()
{
	if (const UEdMode* EdMode = GetModeManager()->GetActiveScriptableMode(Info.ID))
	{
		UContextObjectStore* ContextObjectStore = EdMode->GetInteractiveToolsContext()->ContextObjectStore;
		ContextObjectStore->RemoveContextObject(AnimationEditModeContext.Get());
	}
	
	FEdMode::Exit();
}

void FAnimationEditMode::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdMode::AddReferencedObjects(Collector);
	Collector.AddReferencedObject(AnimationEditModeContext);
}
