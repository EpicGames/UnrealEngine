// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/UObjectHierarchyFwd.h"

#include "AnimationEditContext.generated.h"

UINTERFACE(MinimalAPI)
class UAnimationEditContext : public UInterface
{
	GENERATED_BODY()
};

class IAnimationEditContext
{
	GENERATED_BODY()
public:
	/** 
	 * Get a camera target used to focus the viewport on an object when a user presses 'F' (default). 
	 * @param	 OutTarget	 The target object
	 * @return true if the target sphere was filled-in
	 */
	virtual bool GetCameraTarget(FSphere& OutTarget) const { return false; }

	/** @return the anim preview scene */
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const = 0;

	/**
	* Function to collect strings from nodes to display in the viewport.
	* Use this rather than DrawHUD when adding general text to the viewport.
	* Display of this assumes that this will be mostly used by skeletal controls.
	* @param	OutDebugInfo	Text to display
	*/
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const {}
};
