// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationEditContext.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdMode.h"
#include "Math/Sphere.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AnimationEditMode.generated.h"

#define UE_API ANIMATIONEDITMODE_API

class FAnimationEditMode;
class FText;

/**
 *	A compatibility context object to support IPersonaEditMode-based code. It simply calls into a different
 *	IAnimationEditContext in its implementations.
 */
UCLASS(MinimalAPI)
class UAnimationEditModeContext : public UObject, public IAnimationEditContext
{
	GENERATED_BODY()
public:
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override;
private:
	IAnimationEditContext* EditMode = nullptr;
	static UAnimationEditModeContext* CreateFor(IAnimationEditContext* InEditMode)
	{
		UAnimationEditModeContext* NewPersonaContext = NewObject<UAnimationEditModeContext>();
		NewPersonaContext->EditMode = InEditMode;
		return NewPersonaContext;
	}
	friend FAnimationEditMode;
};

class FAnimationEditMode : public FEdMode, public IAnimationEditContext
{
public:
	UE_API FAnimationEditMode();

	FAnimationEditMode(const FAnimationEditMode&) = delete;
	FAnimationEditMode& operator=(const FAnimationEditMode&) = delete;

	UE_API virtual void Enter() override;
	UE_API virtual void Exit() override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
private:
	TObjectPtr<UAnimationEditModeContext> AnimationEditModeContext;
};

#undef UE_API
